// Copyright Epic Games, Inc. All Rights Reserved.
/*
    Oodle WAVE is a simple data transform on PCM audio data that is modifies the data to be more compressible by Oodle. 
*/
#pragma once

#ifdef __cplusplus
extern "C" {
#endif

// Encodes/decodes 16-bit samples. samples is the input/output buffer, scratch_buffer is a temporary buffer.  
// operates in place. samples is both input and output.
// scratch_buffer should be the same size as samples.
// num_samples is the number of samples in the buffer
// num_channels is the number of channels in the buffer.
void uewav_encode16(short *samples, short *scratch_buffer, long long num_samples, long long num_channels);
void uewav_decode16(short *samples, short *scratch_buffer, long long num_samples, long long num_channels);

#ifdef __cplusplus
}
#endif