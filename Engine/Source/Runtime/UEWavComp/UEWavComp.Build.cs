// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;
using System.IO;

public class UEWavComp : ModuleRules
{
	public UEWavComp(ReadOnlyTargetRules Target) : base(Target)
	{
		Type = ModuleType.External;

		bool bWithLib = false;
		if (Target.Platform == UnrealTargetPlatform.Win64 && Target.WindowsPlatform.Architecture != UnrealArch.Arm64)
		{
			string LibPath = Path.Combine(ModuleDirectory, "Lib");
			PublicAdditionalLibraries.Add(Path.Combine(LibPath, "uewav_w64.lib"));
			bWithLib = true;
		}
		else if (Target.Platform == UnrealTargetPlatform.Win64 && Target.WindowsPlatform.Architecture == UnrealArch.Arm64)
		{
			string LibPath = Path.Combine(ModuleDirectory, "Lib");
			PublicAdditionalLibraries.Add(Path.Combine(LibPath, "uewav_warm64.lib"));
			bWithLib = true;
		}
		else if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			string LibPath = Path.Combine(ModuleDirectory, "Lib");
			PublicAdditionalLibraries.Add(Path.Combine(LibPath, "uewav_mac.a"));
			bWithLib = true;
		}
		else if (Target.IsInPlatformGroup(UnrealPlatformGroup.Unix))
		{
			string LibPath = Path.Combine(ModuleDirectory, "Lib/Unix", Target.Architecture.LinuxName);
			PublicAdditionalLibraries.Add(Path.Combine(LibPath, "uewav_linux.a"));
			bWithLib = true;
		}

		if (bWithLib)
		{
			PublicSystemIncludePaths.Add(Path.Combine(ModuleDirectory, "Source"));
		}

		PublicDefinitions.Add("WITH_OOWAV=" + (bWithLib ? '1' : '0'));
	}
}
